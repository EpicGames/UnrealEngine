// Copyright Epic Games, Inc. All Rights Reserved.

#include "ITimeSlider.h"
#include "AnimatedRange.h"
#include "Misc/FrameTime.h"

FAnimatedRange ITimeSliderController::GetViewRange() const
{
	return FAnimatedRange();
}

FAnimatedRange ITimeSliderController::GetClampRange() const
{
	return FAnimatedRange();
}

TRange<FFrameNumber> ITimeSliderController::GetPlayRange() const
{
	return TRange<FFrameNumber>();
}

TRange<FFrameNumber> ITimeSliderController::GetTimeBounds() const
{
	return TRange<FFrameNumber>();
}

TRange<FFrameNumber> ITimeSliderController::GetSelectionRange() const
{
	return TRange<FFrameNumber>();
}

FFrameTime ITimeSliderController::GetScrubPosition() const
{
	return FFrameTime();
}

void ITimeSliderController::SetScrubPosition(FFrameTime InTime, bool bEvaluate)
{
}

void  ITimeSliderController::SetStoppedPosition(FFrameTime InTime)
{
}

void ITimeSliderController::SetViewRange(double NewRangeMin, double NewRangeMax, EViewRangeInterpolation Interpolation)
{
}

void ITimeSliderController::SetClampRange(double NewRangeMin, double NewRangeMax)
{
}

void ITimeSliderController::SetPlayRange(FFrameNumber RangeStart, int32 RangeDuration)
{
}

void ITimeSliderController::SetSelectionRange(const TRange<FFrameNumber>& NewRange)
{
}

void ITimeSliderController::SetPlaybackStatus(ETimeSliderPlaybackStatus InStatus)
{
}

ETimeSliderPlaybackStatus ITimeSliderController::GetPlaybackStatus() const
{
	return ETimeSliderPlaybackStatus::Stopped;
}