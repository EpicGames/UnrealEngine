// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerTimeSlider.h"

#include "Math/UnrealMathSSE.h"
#include "Widgets/SCompoundWidget.h"

class FSlateRect;
class FWidgetStyle;
struct FGeometry;
struct FPointerEvent;

#define LOCTEXT_NAMESPACE "STimeSlider"


void SSequencerTimeSlider::Construct( const SSequencerTimeSlider::FArguments& InArgs, TSharedRef<ITimeSliderController> InTimeSliderController )
{
	TimeSliderController = InTimeSliderController;
	bMirrorLabels = InArgs._MirrorLabels;
}

int32 SSequencerTimeSlider::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	int32 NewLayer = TimeSliderController->OnPaintTimeSlider( bMirrorLabels, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled );

	return FMath::Max( NewLayer, SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, NewLayer, InWidgetStyle, ShouldBeEnabled( bParentEnabled ) ) );
}

void SSequencerTimeSlider::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TimeSliderController->OnMouseEnter( *this, MyGeometry, MouseEvent );
}

void SSequencerTimeSlider::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	TimeSliderController->OnMouseLeave( *this, MouseEvent );
}

FReply SSequencerTimeSlider::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	TimeSliderController->OnTimeSliderMouseButtonDown( *this, MyGeometry, MouseEvent );
	return FReply::Handled().CaptureMouse(AsShared()).PreventThrottling();
}

FReply SSequencerTimeSlider::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return TimeSliderController->OnTimeSliderMouseButtonUp( *this,  MyGeometry, MouseEvent );
}

FReply SSequencerTimeSlider::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return TimeSliderController->OnTimeSliderMouseMove( *this, MyGeometry, MouseEvent );
}

FReply SSequencerTimeSlider::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return TimeSliderController->OnMouseButtonDoubleClick( *this, MyGeometry, MouseEvent );
}

FVector2D SSequencerTimeSlider::ComputeDesiredSize( float ) const
{
	return FVector2D(100, TimeSliderController->ComputeHeight());
}

FReply SSequencerTimeSlider::OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return TimeSliderController->OnTimeSliderMouseWheel( *this, MyGeometry, MouseEvent );
}

FCursorReply SSequencerTimeSlider::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	return TimeSliderController->OnCursorQuery( *this, MyGeometry, CursorEvent );
}

void SSequencerTimeSlider::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	TimeSliderController->OnFocusLost( *this, InFocusEvent );
}

#undef LOCTEXT_NAMESPACE
