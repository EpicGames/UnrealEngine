// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "ITimeSlider.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
struct FGeometry;
struct FPointerEvent;

class SSequencerTimeSlider : public ITimeSlider
{
public:

	SLATE_BEGIN_ARGS(SSequencerTimeSlider)
		: _MirrorLabels( false )
	{}
		/* If we should mirror the labels on the timeline */
		SLATE_ARGUMENT( bool, MirrorLabels )
	SLATE_END_ARGS()


	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs, TSharedRef<ITimeSliderController> InTimeSliderController );

protected:
	//~ Begin SWidget
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;
	//~ End SWidget

private:
	TSharedPtr<ITimeSliderController> TimeSliderController;
	bool bMirrorLabels;
};

