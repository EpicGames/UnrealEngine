// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimeRange.h"

#include "AnimatedRange.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/Range.h"
#include "Misc/Attribute.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/Optional.h"
#include "MovieSceneTimeHelpers.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Types/SlateStructs.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SNullWidget.h"

class SWidget;
template <typename NumericType> struct INumericTypeInterface;

#define LOCTEXT_NAMESPACE "STimeRange"

void STimeRange::Construct( const STimeRange::FArguments& InArgs, TSharedRef<ITimeSliderController> InTimeSliderController )
{
	TimeSliderController = InTimeSliderController;
	PlaybackRangeStartColor = InArgs._PlaybackRangeStartColor;
	PlaybackRangeEndColor = InArgs._PlaybackRangeEndColor;

	TSharedRef<SWidget> WorkingRangeStart = SNullWidget::NullWidget, WorkingRangeEnd = SNullWidget::NullWidget;
	if (InArgs._ShowWorkingRange)
	{
		WorkingRangeStart = SNew(SSpinBox<double>)
		.IsEnabled(InArgs._EnableWorkingRange)
		.Value(this, &STimeRange::WorkingStartTime)
		.ToolTipText(LOCTEXT("WorkingRangeStart", "Working Range Start"))
		.OnValueCommitted(this, &STimeRange::OnWorkingStartTimeCommitted)
		.OnValueChanged(this, &STimeRange::OnWorkingStartTimeChanged)
		.MinValue(TOptional<double>())
		.MaxValue(TOptional<double>())
		.Style(&FAppStyle::Get().GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
		.TypeInterface(InArgs._NumericTypeInterface)
		.ClearKeyboardFocusOnCommit(true)
		.Delta(this, &STimeRange::GetSpinboxDelta)
		.LinearDeltaSensitivity(25);

		WorkingRangeEnd = SNew(SSpinBox<double>)
		.IsEnabled(InArgs._EnableWorkingRange)
		.Value(this, &STimeRange::WorkingEndTime)
		.ToolTipText(LOCTEXT("WorkingRangeEnd", "Working Range End"))
		.OnValueCommitted( this, &STimeRange::OnWorkingEndTimeCommitted )
		.OnValueChanged( this, &STimeRange::OnWorkingEndTimeChanged )
		.MinValue(TOptional<double>())
		.MaxValue(TOptional<double>())
		.Style(&FAppStyle::Get().GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
		.TypeInterface(InArgs._NumericTypeInterface)
		.ClearKeyboardFocusOnCommit(true)
		.Delta(this, &STimeRange::GetSpinboxDelta)
		.LinearDeltaSensitivity(25);
	}

	TSharedRef<SWidget> ViewRangeStart = SNullWidget::NullWidget, ViewRangeEnd = SNullWidget::NullWidget;
	if (InArgs._ShowViewRange)
	{
		ViewRangeStart = SNew(SSpinBox<double>)
		.IsEnabled(InArgs._EnableViewRange)
		.Value(this, &STimeRange::ViewStartTime)
		.ToolTipText(LOCTEXT("ViewStartTimeTooltip", "View Range Start Time"))
		.OnValueCommitted( this, &STimeRange::OnViewStartTimeCommitted )
		.OnValueChanged( this, &STimeRange::OnViewStartTimeChanged )
		.MinValue(TOptional<double>())
		.MaxValue(TOptional<double>())
		.Style(&FAppStyle::Get().GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
		.TypeInterface(InArgs._NumericTypeInterface)
		.ClearKeyboardFocusOnCommit(true)
		.Delta(this, &STimeRange::GetSpinboxDelta)
		.LinearDeltaSensitivity(25);


		ViewRangeEnd = SNew(SSpinBox<double>)
		.IsEnabled(InArgs._EnableViewRange)
		.Value(this, &STimeRange::ViewEndTime)
		.ToolTipText(LOCTEXT("ViewEndTimeTooltip", "View Range End Time"))
		.OnValueCommitted( this, &STimeRange::OnViewEndTimeCommitted )
		.OnValueChanged( this, &STimeRange::OnViewEndTimeChanged )
		.MinValue(TOptional<double>())
		.MaxValue(TOptional<double>())
		.Style(&FAppStyle::Get().GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
		.TypeInterface(InArgs._NumericTypeInterface)
		.ClearKeyboardFocusOnCommit(true)
		.Delta(this, &STimeRange::GetSpinboxDelta)
		.LinearDeltaSensitivity(25);
	}

	TSharedRef<SWidget> PlaybackRangeStart = SNullWidget::NullWidget, PlaybackRangeEnd = SNullWidget::NullWidget;
	if (InArgs._ShowPlaybackRange)
	{
		PlaybackRangeStart = SNew(SSpinBox<double>)
		.IsEnabled(InArgs._EnablePlaybackRange)
		.Value(this, &STimeRange::PlayStartTime)
		.ToolTipText(LOCTEXT("PlayStartTimeTooltip", "Playback Range Start Time"))
		.OnValueCommitted(this, &STimeRange::OnPlayStartTimeCommitted)
		.OnValueChanged(this, &STimeRange::OnPlayStartTimeChanged)
		.MinValue(TOptional<double>())
		.MaxValue(TOptional<double>())
		.Style(&FAppStyle::Get().GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
		.TypeInterface(InArgs._NumericTypeInterface)
		.ClearKeyboardFocusOnCommit(true)
		.Delta(this, &STimeRange::GetSpinboxDelta)
		.LinearDeltaSensitivity(25);


		PlaybackRangeEnd = SNew(SSpinBox<double>)
		.IsEnabled(InArgs._EnablePlaybackRange)
		.Value(this, &STimeRange::PlayEndTime)
		.ToolTipText(LOCTEXT("PlayEndTimeTooltip", "Playback Range Stop Time"))
		.OnValueCommitted( this, &STimeRange::OnPlayEndTimeCommitted )
		.OnValueChanged( this, &STimeRange::OnPlayEndTimeChanged )
		.MinValue(TOptional<double>())
		.MaxValue(TOptional<double>())
		.Style(&FAppStyle::Get().GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
		.TypeInterface(InArgs._NumericTypeInterface)
		.ClearKeyboardFocusOnCommit(true)
		.Delta(this, &STimeRange::GetSpinboxDelta)
		.LinearDeltaSensitivity(25);
	}

	this->ChildSlot
	.HAlign(HAlign_Fill)
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(2.f)
		[
			SNew(SBox)
			.Visibility(InArgs._ShowWorkingRange ? EVisibility::Visible : EVisibility::Collapsed)
			.MinDesiredWidth(64.f)
			.HAlign(HAlign_Center)
			[
				WorkingRangeStart
			]
		]

		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(2.f)
		[
			SNew(SBox)
			.MinDesiredWidth(64.f)
			.HAlign(HAlign_Center)
			.Visibility(InArgs._ShowPlaybackRange ? EVisibility::Visible : EVisibility::Collapsed)
			[
				SNew(SBorder)
				.Padding(0.f)
				.BorderImage(nullptr)
				.ForegroundColor(PlaybackRangeStartColor)
				[
					PlaybackRangeStart
				]
			]
		]

		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(2.f)
		[
			SNew(SBox)
			.Visibility(InArgs._ShowViewRange ? EVisibility::Visible : EVisibility::Collapsed)
			.MinDesiredWidth(64.f)
			.HAlign(HAlign_Center)
			[
				ViewRangeStart
			]
		]

		+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(2.0f, 4.0f)
			.VAlign(VAlign_Center)
			[
				InArgs._CenterContent.Widget
			]
		
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(2.f)
		[
			SNew(SBox)
			.Visibility(InArgs._ShowViewRange ? EVisibility::Visible : EVisibility::Collapsed)
			.MinDesiredWidth(64.f)
			.HAlign(HAlign_Center)
			[
				ViewRangeEnd
			]
		]

		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(2.f)
		[
			SNew(SBox)
			.MinDesiredWidth(64.f)
			.HAlign(HAlign_Center)
			.Visibility(InArgs._ShowPlaybackRange ? EVisibility::Visible : EVisibility::Collapsed)
			[
				SNew(SBorder)
				.Padding(0.f)
				.BorderImage(nullptr)
				.ForegroundColor(PlaybackRangeEndColor)
				[
					PlaybackRangeEnd
				]
			]
		]

		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(2.f)
		[
			SNew(SBox)
			.MinDesiredWidth(64.f)
			.HAlign(HAlign_Center)
			.Visibility(InArgs._ShowWorkingRange ? EVisibility::Visible : EVisibility::Collapsed)
			[
				WorkingRangeEnd
			]
		]
	];
}

double STimeRange::WorkingStartTime() const
{
	FFrameRate Rate = TimeSliderController->GetTickResolution();
	FFrameTime Time = TimeSliderController->GetClampRange().GetLowerBoundValue() * Rate;
	return Time.GetFrame().Value;
}

double STimeRange::WorkingEndTime() const
{
	FFrameRate Rate = TimeSliderController->GetTickResolution();
	FFrameTime Time = TimeSliderController->GetClampRange().GetUpperBoundValue() * Rate;
	return Time.GetFrame().Value;
}

double STimeRange::ViewStartTime() const
{
	FFrameRate TickResolution = TimeSliderController->GetTickResolution();

	// View range is in seconds so we convert it to tick resolution
	FFrameTime Time = TimeSliderController->GetViewRange().GetLowerBoundValue() * TickResolution;
	return Time.GetFrame().Value;
}

double STimeRange::ViewEndTime() const
{
	FFrameRate TickResolution = TimeSliderController->GetTickResolution();

	// View range is in seconds so we convert it to tick resolution
	FFrameTime Time = TimeSliderController->GetViewRange().GetUpperBoundValue() * TickResolution;
	return Time.GetFrame().Value;
}

double STimeRange::GetSpinboxDelta() const
{
	return TimeSliderController->GetTickResolution().AsDecimal() * TimeSliderController->GetDisplayRate().AsInterval();
}

double STimeRange::PlayStartTime() const
{
	FFrameNumber LowerBound = UE::MovieScene::DiscreteInclusiveLower(TimeSliderController->GetPlayRange());
	return LowerBound.Value; 
}

double STimeRange::PlayEndTime() const
{
	FFrameNumber UpperBound = UE::MovieScene::DiscreteExclusiveUpper(TimeSliderController->GetPlayRange());
	return UpperBound.Value;
}

void STimeRange::OnWorkingStartTimeCommitted(double NewValue, ETextCommit::Type InTextCommit)
{
	OnWorkingStartTimeChanged(NewValue);
}

void STimeRange::OnWorkingEndTimeCommitted(double NewValue, ETextCommit::Type InTextCommit)
{
	OnWorkingEndTimeChanged(NewValue);
}

void STimeRange::OnViewStartTimeCommitted(double NewValue, ETextCommit::Type InTextCommit)
{
	OnViewStartTimeChanged(NewValue);
}

void STimeRange::OnViewEndTimeCommitted(double NewValue, ETextCommit::Type InTextCommit)
{
	OnViewEndTimeChanged(NewValue);
}

void STimeRange::OnPlayStartTimeCommitted(double NewValue, ETextCommit::Type InTextCommit)
{
	OnPlayStartTimeChanged(NewValue);
}

void STimeRange::OnPlayEndTimeCommitted(double NewValue, ETextCommit::Type InTextCommit)
{
	OnPlayEndTimeChanged(NewValue);
}

void STimeRange::OnWorkingStartTimeChanged(double NewValue)
{
	FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	double Time = TickResolution.AsSeconds(FFrameTime::FromDecimal(NewValue));

	// Clamp range is in seconds
	TimeSliderController->SetClampRange(Time, TimeSliderController->GetClampRange().GetUpperBoundValue());

	if (Time > TimeSliderController->GetViewRange().GetLowerBoundValue())
	{
		OnViewStartTimeChanged(NewValue);
	}
}

void STimeRange::OnWorkingEndTimeChanged(double NewValue)
{
	FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	double Time = TickResolution.AsSeconds(FFrameTime::FromDecimal(NewValue));

	// Clamp range is in seconds
	TimeSliderController->SetClampRange(TimeSliderController->GetClampRange().GetLowerBoundValue(), Time);

	if (Time < TimeSliderController->GetViewRange().GetUpperBoundValue())
	{
		OnViewEndTimeChanged(NewValue);
	}
}

void STimeRange::OnViewStartTimeChanged(double NewValue)
{
	FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	double Time = TickResolution.AsSeconds(FFrameTime::FromDecimal(NewValue));

	double ViewStartTime = TimeSliderController->GetViewRange().GetLowerBoundValue();
	double ViewEndTime = TimeSliderController->GetViewRange().GetUpperBoundValue();

	double ClampStartTime = TimeSliderController.Get()->GetClampRange().GetLowerBoundValue();
	double ClampEndTime = TimeSliderController.Get()->GetClampRange().GetUpperBoundValue();

	if (Time >= ViewEndTime)
	{
		double ViewDuration = ViewEndTime - ViewStartTime;
		ViewEndTime = Time + ViewDuration;

		if (ViewEndTime > TimeSliderController.Get()->GetClampRange().GetUpperBoundValue())
		{
			TimeSliderController->SetClampRange(TimeSliderController->GetClampRange().GetLowerBoundValue(), ViewEndTime);
		}
	}


	if (Time < TimeSliderController.Get()->GetClampRange().GetLowerBoundValue())
	{
		TimeSliderController->SetClampRange(Time, TimeSliderController->GetClampRange().GetUpperBoundValue());
	}

	TimeSliderController->SetViewRange(Time, ViewEndTime, EViewRangeInterpolation::Immediate);
}


void STimeRange::OnViewEndTimeChanged(double NewValue)
{
	FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	double Time = TickResolution.AsSeconds(FFrameTime::FromDecimal(NewValue));

	double ViewStartTime = TimeSliderController->GetViewRange().GetLowerBoundValue();
	double ViewEndTime = TimeSliderController->GetViewRange().GetUpperBoundValue();

	if (Time <= ViewStartTime)
	{
		double ViewDuration = ViewEndTime - ViewStartTime;
		ViewStartTime = Time - ViewDuration;

		if (ViewStartTime < TimeSliderController.Get()->GetClampRange().GetLowerBoundValue())
		{
			TimeSliderController->SetClampRange(ViewStartTime, TimeSliderController->GetClampRange().GetUpperBoundValue());
		}
	}

	if (Time > TimeSliderController->GetClampRange().GetUpperBoundValue())
	{
		TimeSliderController->SetClampRange(TimeSliderController->GetClampRange().GetLowerBoundValue(), Time);
	}

	TimeSliderController->SetViewRange(ViewStartTime, Time, EViewRangeInterpolation::Immediate);
}

void STimeRange::OnPlayStartTimeChanged(double NewValue)
{
	FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	FFrameTime Time = FFrameTime::FromDecimal(NewValue);
	double     TimeInSeconds = TickResolution.AsSeconds(Time);

	TRange<FFrameNumber> PlayRange = TimeSliderController->GetPlayRange();
	FFrameNumber PlayDuration;
	if (Time.FrameNumber >= UE::MovieScene::DiscreteExclusiveUpper(PlayRange))
	{
		PlayDuration = UE::MovieScene::DiscreteExclusiveUpper(PlayRange) - UE::MovieScene::DiscreteInclusiveLower(PlayRange);
	}
	else
	{
		PlayDuration = UE::MovieScene::DiscreteExclusiveUpper(PlayRange) - Time.FrameNumber;
	}

	TimeSliderController->SetPlayRange(Time.FrameNumber, PlayDuration.Value);

	// Expand view ranges if outside of play range
	if (TimeInSeconds < TimeSliderController.Get()->GetClampRange().GetLowerBoundValue())
	{
		OnViewStartTimeChanged(NewValue);
	}

	FFrameNumber PlayEnd = TimeSliderController->GetPlayRange().GetUpperBoundValue();
	double PlayEndSeconds = PlayEnd / TickResolution;

	if (PlayEndSeconds > TimeSliderController.Get()->GetClampRange().GetUpperBoundValue())
	{
		OnViewEndTimeChanged(TickResolution.AsFrameNumber(PlayEndSeconds).Value);
	}
}

void STimeRange::OnPlayEndTimeChanged(double NewValue)
{
	FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	FFrameTime Time = FFrameTime::FromDecimal(NewValue);
	double     TimeInSeconds = TickResolution.AsSeconds(Time);

	TRange<FFrameNumber> PlayRange = TimeSliderController->GetPlayRange();
	FFrameNumber PlayDuration;
	FFrameNumber StartFrame = UE::MovieScene::DiscreteInclusiveLower(PlayRange);
	if (Time.FrameNumber <= StartFrame)
	{
		PlayDuration = UE::MovieScene::DiscreteExclusiveUpper(PlayRange) - StartFrame;
		StartFrame = Time.FrameNumber - PlayDuration;
	}
	else
	{
		PlayDuration = Time.FrameNumber - StartFrame;
	}

	TimeSliderController->SetPlayRange(StartFrame, PlayDuration.Value);

	// Expand view ranges if outside of play range
	if (TimeInSeconds > TimeSliderController->GetClampRange().GetUpperBoundValue())
	{
		OnViewEndTimeChanged(NewValue);
	}

	FFrameNumber PlayStart = TimeSliderController->GetPlayRange().GetLowerBoundValue();
	double PlayStartSeconds = PlayStart / TickResolution;

	if (PlayStartSeconds < TimeSliderController.Get()->GetClampRange().GetLowerBoundValue())
	{
		OnViewStartTimeChanged(TickResolution.AsFrameNumber(PlayStartSeconds).Value);
	}
}

#undef LOCTEXT_NAMESPACE
