// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITimeSlider.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SWidget;
template <typename NumericType> struct INumericTypeInterface;

class STimeRange : public ITimeSlider
{
public:
	SLATE_BEGIN_ARGS(STimeRange)
		: _ShowWorkingRange(true)
		, _ShowViewRange(false)
		, _ShowPlaybackRange(true)
		, _PlaybackRangeStartColor(FLinearColor::Red)
		, _PlaybackRangeEndColor(FLinearColor::Green)
	{}
		/** Whether to show the working range */
		SLATE_ATTRIBUTE( TSharedPtr<INumericTypeInterface<double>>, NumericTypeInterface )
		/** Whether to show the working range */
		SLATE_ARGUMENT( bool, ShowWorkingRange )
		/** Whether to show the view range */
		SLATE_ARGUMENT( bool, ShowViewRange )
		/** Whether to show the playback range */
		SLATE_ARGUMENT( bool, ShowPlaybackRange )
		/** Whether to enable the working range */
		SLATE_ARGUMENT( bool, EnableWorkingRange )
		/** Whether to enable the view range */
		SLATE_ARGUMENT( bool, EnableViewRange )
		/** Whether to enable the playback range */
		SLATE_ARGUMENT( bool, EnablePlaybackRange )
		/** The color of the playback start indicator */
		SLATE_ARGUMENT( FLinearColor, PlaybackRangeStartColor )
		/** The color of the playback end indicator */
		SLATE_ARGUMENT( FLinearColor, PlaybackRangeEndColor )
		/* Content to display inside the time range */
		SLATE_DEFAULT_SLOT( FArguments, CenterContent )
	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs, TSharedRef<ITimeSliderController> InTimeSliderController);

protected:
	double GetSpinboxDelta() const;
	
protected:

	double PlayStartTime() const;
	double PlayEndTime() const;

	void OnPlayStartTimeCommitted(double NewValue, ETextCommit::Type InTextCommit);
	void OnPlayEndTimeCommitted(double NewValue, ETextCommit::Type InTextCommit);

	void OnPlayStartTimeChanged(double NewValue);
	void OnPlayEndTimeChanged(double NewValue);

protected:

	double ViewStartTime() const;
	double ViewEndTime() const;
	
	void OnViewStartTimeCommitted(double NewValue, ETextCommit::Type InTextCommit);
	void OnViewEndTimeCommitted(double NewValue, ETextCommit::Type InTextCommit);

	void OnViewStartTimeChanged(double NewValue);
	void OnViewEndTimeChanged(double NewValue);

protected:

	double WorkingStartTime() const;
	double WorkingEndTime() const;

	void OnWorkingStartTimeCommitted(double NewValue, ETextCommit::Type InTextCommit);
	void OnWorkingEndTimeCommitted(double NewValue, ETextCommit::Type InTextCommit);

	void OnWorkingStartTimeChanged(double NewValue);
	void OnWorkingEndTimeChanged(double NewValue);

private:
	TSharedPtr<ITimeSliderController> TimeSliderController;

	FLinearColor PlaybackRangeStartColor;
	FLinearColor PlaybackRangeEndColor;
};
