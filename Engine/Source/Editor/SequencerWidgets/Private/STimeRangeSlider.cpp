// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimeRangeSlider.h"

#include "AnimatedRange.h"
#include "ITimeSlider.h"
#include "Input/Events.h"
#include "Layout/Geometry.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/WidgetStyle.h"
#include "UObject/NameTypes.h"

class FSlateRect;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "STimeRangeSlider"

namespace TimeRangeSliderConstants
{
	const int32 HandleSize = 14;
	const int32 MinimumScrubberWidth = HandleSize * 2;
}

void STimeRangeSlider::Construct( const FArguments& InArgs, TSharedRef<ITimeSliderController> InTimeSliderController)
{
	TimeSliderController = InTimeSliderController;
	LastViewRange = TimeSliderController->GetViewRange();

	ResetState();
	ResetHoveredState();
}

double STimeRangeSlider::ComputeDragDelta(const FPointerEvent& MouseEvent, double GeometryWidth) const
{
	double StartTime = 0;
	double EndTime = 0;

	if (TimeSliderController.IsValid())
	{
		StartTime = TimeSliderController->GetClampRange().GetLowerBoundValue();
		EndTime = TimeSliderController->GetClampRange().GetUpperBoundValue();
	}
	double DragDistance = (MouseEvent.GetScreenSpacePosition() - MouseDownPosition).X;

	const double Width = (GeometryWidth - TimeRangeSliderConstants::HandleSize * 2);
	const double PixelToUnits = FMath::IsNearlyZero(Width) ? 0 : (EndTime - StartTime) / Width;
	return DragDistance * PixelToUnits;
}

void STimeRangeSlider::ComputeHandleOffsets(double& LeftHandleOffset, double& HandleOffset, double& RightHandleOffset, double GeometryWidth) const
{
	double StartTime = 0;
	double InTime = 0;
	double OutTime = 0;
	double EndTime = 0;

	if (TimeSliderController.IsValid())
	{
		StartTime = TimeSliderController->GetClampRange().GetLowerBoundValue();
		InTime = TimeSliderController->GetViewRange().GetLowerBoundValue();
		OutTime = TimeSliderController->GetViewRange().GetUpperBoundValue();
		EndTime = TimeSliderController->GetClampRange().GetUpperBoundValue();
	}

	const double UnitsToPixel = (GeometryWidth - TimeRangeSliderConstants::HandleSize*2) / (EndTime - StartTime);

	LeftHandleOffset = (InTime - StartTime) * UnitsToPixel;
	HandleOffset = LeftHandleOffset + TimeRangeSliderConstants::HandleSize;
	RightHandleOffset = HandleOffset + (OutTime - InTime) * UnitsToPixel;
	
	double ScrubberWidth = RightHandleOffset-LeftHandleOffset-TimeRangeSliderConstants::HandleSize;
	if (ScrubberWidth < (double)TimeRangeSliderConstants::MinimumScrubberWidth)
	{
		HandleOffset = HandleOffset - (TimeRangeSliderConstants::MinimumScrubberWidth - ScrubberWidth) / 2.0;
		LeftHandleOffset = HandleOffset - TimeRangeSliderConstants::HandleSize;
		RightHandleOffset = HandleOffset + TimeRangeSliderConstants::MinimumScrubberWidth;
	}
}


FVector2D STimeRangeSlider::ComputeDesiredSize(float) const
{
	return FVector2D(4.0f * TimeRangeSliderConstants::HandleSize, TimeRangeSliderConstants::HandleSize);
}


int32 STimeRangeSlider::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	const int32 BackgroundLayer = LayerId+1;
	const int32 SliderBoxLayer = BackgroundLayer+1;
	const int32 HandleLayer = SliderBoxLayer+1;

	static const FSlateBrush* RangeHandleLeft = FAppStyle::GetBrush( TEXT( "Sequencer.Timeline.RangeHandleLeft" ) ); 
	static const FSlateBrush* RangeHandleRight = FAppStyle::GetBrush( TEXT( "Sequencer.Timeline.RangeHandleRight" ) ); 
	static const FSlateBrush* RangeHandle = FAppStyle::GetBrush( TEXT( "Sequencer.Timeline.RangeHandle" ) ); 

	double LeftHandleOffset = 0;
	double HandleOffset = 0;
	double RightHandleOffset = 0;
	ComputeHandleOffsets(LeftHandleOffset, HandleOffset, RightHandleOffset, AllottedGeometry.GetLocalSize().X);

	static const FName SelectionColorName("SelectionColor");
	FLinearColor SelectionColor = FAppStyle::GetSlateColor(SelectionColorName).GetColor(FWidgetStyle());

	// Draw the handle box
	FSlateDrawElement::MakeBox( 
		OutDrawElements,
		LayerId, 
		AllottedGeometry.ToPaintGeometry(FVector2f(RightHandleOffset-LeftHandleOffset-TimeRangeSliderConstants::HandleSize, TimeRangeSliderConstants::HandleSize), FSlateLayoutTransform(FVector2f(HandleOffset, 0.0f))),
		RangeHandle,
		ESlateDrawEffect::None,
		(bHandleDragged || bHandleHovered) ? SelectionColor : FLinearColor::Gray);

	// Draw the left handle box
	FSlateDrawElement::MakeBox( 
		OutDrawElements,
		LayerId, 
		AllottedGeometry.ToPaintGeometry(FVector2f(TimeRangeSliderConstants::HandleSize, TimeRangeSliderConstants::HandleSize), FSlateLayoutTransform(FVector2f(LeftHandleOffset, 0.0f))),
		RangeHandleLeft,
		ESlateDrawEffect::None,
		(bLeftHandleDragged || bLeftHandleHovered) ? SelectionColor : FLinearColor::Gray);

	// Draw the right handle box
	FSlateDrawElement::MakeBox( 
		OutDrawElements,
		LayerId, 
		AllottedGeometry.ToPaintGeometry(FVector2f(TimeRangeSliderConstants::HandleSize, TimeRangeSliderConstants::HandleSize), FSlateLayoutTransform(FVector2f(RightHandleOffset, 0.0f))),
		RangeHandleRight,
		ESlateDrawEffect::None,
		(bRightHandleDragged || bRightHandleHovered) ? SelectionColor : FLinearColor::Gray);

	SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, ShouldBeEnabled( bParentEnabled ));

	return LayerId;
}

FReply STimeRangeSlider::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	MouseDownPosition = MouseEvent.GetScreenSpacePosition();
	if (TimeSliderController.IsValid())
	{		
		MouseDownViewRange = TimeSliderController->GetViewRange();
	}

	if (bHandleHovered)
	{
		bHandleDragged = true;
		return FReply::Handled().CaptureMouse(AsShared());
	}
	else if (bLeftHandleHovered)
	{
		bLeftHandleDragged = true;
		return FReply::Handled().CaptureMouse(AsShared());
	}
	else if (bRightHandleHovered)
	{
		bRightHandleDragged = true;
		return FReply::Handled().CaptureMouse(AsShared());
	}

	return FReply::Unhandled();
}

FReply STimeRangeSlider::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	ResetState();

	return FReply::Handled().ReleaseMouseCapture();
}

FReply STimeRangeSlider::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (HasMouseCapture())
	{
		double DragDelta = ComputeDragDelta(MouseEvent, MyGeometry.GetAbsoluteSize().X);

		ITimeSliderController* TimeSliderControllerPtr = TimeSliderController.Get();
		if (!TimeSliderControllerPtr)
		{
			return FReply::Handled();
		}

		if (bHandleDragged)
		{
			double NewIn = MouseDownViewRange.GetLowerBoundValue() + DragDelta;
			double NewOut = MouseDownViewRange.GetUpperBoundValue() + DragDelta;

			TRange<double> ClampRange = TimeSliderControllerPtr->GetClampRange();
			if (NewIn < ClampRange.GetLowerBoundValue())
			{
				NewIn = ClampRange.GetLowerBoundValue();
				NewOut = NewIn + (MouseDownViewRange.GetUpperBoundValue() - MouseDownViewRange.GetLowerBoundValue());
			}
			else if (NewOut > ClampRange.GetUpperBoundValue())
			{
				NewOut = ClampRange.GetUpperBoundValue();
				NewIn = NewOut - (MouseDownViewRange.GetUpperBoundValue() - MouseDownViewRange.GetLowerBoundValue());
			}
			
			TimeSliderControllerPtr->SetViewRange(NewIn, NewOut, EViewRangeInterpolation::Immediate);
		}
		else if (bLeftHandleDragged || bRightHandleDragged)
		{
			double NewIn = 0;
			double NewOut = 0;

			if (bLeftHandleDragged)
			{
				NewIn = MouseDownViewRange.GetLowerBoundValue() + DragDelta;
				
				NewOut = MouseDownViewRange.GetUpperBoundValue();
				if (MouseEvent.IsShiftDown()) 
				{
					NewOut -= DragDelta;
				}
			}
			else
			{
				NewIn = MouseDownViewRange.GetLowerBoundValue();
				if (MouseEvent.IsShiftDown())
				{
 					NewIn -= DragDelta;
				}

				NewOut = MouseDownViewRange.GetUpperBoundValue() + DragDelta;
			}

			// In cases of extreme zoom the drag delta will be greater than the difference between In/Out.
			// This causes zooming to then become pan at extreme levels which is undesirable.
			if (NewIn >= NewOut)
			{
				return FReply::Handled();
			}

			TimeSliderControllerPtr->SetViewRange(NewIn, NewOut, EViewRangeInterpolation::Immediate);
		}

		return FReply::Handled();
	}
	else
	{
		ResetHoveredState();

		double LeftHandleOffset = 0;
		double HandleOffset = 0;
		double RightHandleOffset = 0;
		ComputeHandleOffsets(LeftHandleOffset, HandleOffset, RightHandleOffset, MyGeometry.GetLocalSize().X);
		
		FGeometry LeftHandleRect  = MyGeometry.MakeChild(FVector2D(TimeRangeSliderConstants::HandleSize, TimeRangeSliderConstants::HandleSize), FSlateLayoutTransform(FVector2D(LeftHandleOffset, 0)));
		FGeometry RightHandleRect = MyGeometry.MakeChild(FVector2D(TimeRangeSliderConstants::HandleSize, TimeRangeSliderConstants::HandleSize), FSlateLayoutTransform(FVector2D(RightHandleOffset, 0)));
		FGeometry HandleRect      = MyGeometry.MakeChild(FVector2D(RightHandleOffset-LeftHandleOffset-TimeRangeSliderConstants::HandleSize, TimeRangeSliderConstants::HandleSize), FSlateLayoutTransform(FVector2D(HandleOffset, 0)));

		FVector2D LocalMousePosition = MouseEvent.GetScreenSpacePosition();

		if (HandleRect.IsUnderLocation(LocalMousePosition))
		{
			bHandleHovered = true;
		}
		else if (LeftHandleRect.IsUnderLocation(LocalMousePosition))
		{
			bLeftHandleHovered = true;
		}
		else if (RightHandleRect.IsUnderLocation(LocalMousePosition))
		{
			bRightHandleHovered = true;
		}
	}
	return FReply::Unhandled();
}

void STimeRangeSlider::OnMouseLeave( const FPointerEvent& MouseEvent )
{
	SCompoundWidget::OnMouseLeave(MouseEvent);

	if (!HasMouseCapture())
	{
		ResetHoveredState();
	}
}

FReply STimeRangeSlider::OnMouseButtonDoubleClick( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	ResetState();

	OnMouseMove(MyGeometry, MouseEvent);

	if (bHandleHovered)
	{
		if (TimeSliderController.IsValid())
		{
			if (FMath::IsNearlyEqual(TimeSliderController->GetViewRange().GetLowerBoundValue(), TimeSliderController->GetClampRange().GetLowerBoundValue()) &&
				FMath::IsNearlyEqual(TimeSliderController->GetViewRange().GetUpperBoundValue(), TimeSliderController->GetClampRange().GetUpperBoundValue()))
			{
				if (!LastViewRange.IsEmpty())
				{
					TimeSliderController->SetViewRange(LastViewRange.GetLowerBoundValue(), LastViewRange.GetUpperBoundValue(), EViewRangeInterpolation::Immediate);
				}
			}
			else
			{
				LastViewRange = TimeSliderController->GetViewRange();
				TimeSliderController->SetViewRange(TimeSliderController->GetClampRange().GetLowerBoundValue(), TimeSliderController->GetClampRange().GetUpperBoundValue(), EViewRangeInterpolation::Immediate);
			}
		}

		ResetState();
		return FReply::Handled();
	}
	ResetState();
	return FReply::Unhandled();
}

void STimeRangeSlider::ResetState()
{
	bHandleDragged = false;
	bLeftHandleDragged = false;
	bRightHandleDragged = false;
	ResetHoveredState();
}

void STimeRangeSlider::ResetHoveredState()
{
	bHandleHovered = false;
	bLeftHandleHovered = false;
	bRightHandleHovered = false;
}

#undef LOCTEXT_NAMESPACE
