// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Math/Range.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
class ITimeSliderController;
class SWidget;
struct FGeometry;
struct FPointerEvent;

class STimeRangeSlider : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STimeRangeSlider){}
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, TSharedRef<ITimeSliderController> InTimeSliderController );

	// SWidget interface
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseButtonDoubleClick( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

protected:
	void ResetState();
	void ResetHoveredState();
	double ComputeDragDelta(const FPointerEvent& MouseEvent, double GeometryWidth) const;
	void ComputeHandleOffsets(double& LeftHandleOffset, double& RightHandleOffset, double&HandleOffset, double GeometryWidth) const;

private:
	/* The left handle is being dragged */
	bool bLeftHandleDragged;
	/* The right handle is being dragged */
	bool bRightHandleDragged;
	/* The handle is being dragged */
	bool bHandleDragged;

	/* The left handle is hovered */
	bool bLeftHandleHovered;
	/* The right handle is hovered */
	bool bRightHandleHovered;
	/* The handle is hovered */
	bool bHandleHovered;

	/* The position of the mouse on mouse down */
	FVector2D MouseDownPosition;

	/* The in/out view range on mouse down */
	TRange<double> MouseDownViewRange;

	/* The in/out view range viewed before expansion */
	TRange<double> LastViewRange;

	TSharedPtr<ITimeSliderController> TimeSliderController;
};
