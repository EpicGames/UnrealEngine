// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISequencerWidgetsModule.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "SSequencerTimeSlider.h"
#include "STimeRange.h"
#include "STimeRangeSlider.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class ITimeSlider;
class ITimeSliderController;
class SWidget;


/**
 * The public interface of SequencerModule
 */
class FSequencerWidgetsModule
	: public ISequencerWidgetsModule
{
public:

	// ISequencerWidgetsModule interface

	TSharedRef<ITimeSlider> CreateTimeSlider(const TSharedRef<ITimeSliderController>& InController, bool bMirrorLabels) override
	{
		return SNew(SSequencerTimeSlider, InController)
			.MirrorLabels(bMirrorLabels);
	}

	TSharedRef<ITimeSlider> CreateTimeSlider(const TSharedRef<ITimeSliderController>& InController, const TAttribute<EVisibility>& VisibilityDelegate, bool bMirrorLabels) override
	{
		return SNew(SSequencerTimeSlider, InController)
			.Visibility(VisibilityDelegate)
			.MirrorLabels(bMirrorLabels);
	}

	TSharedRef<SWidget> CreateTimeRangeSlider( const TSharedRef<class ITimeSliderController>& InController ) override
	{
		return SNew( STimeRangeSlider, InController );
	}

	TSharedRef<ITimeSlider> CreateTimeRange(const FTimeRangeArgs& InArgs, const TSharedRef<SWidget>& Content) override
	{
		return SNew( STimeRange, InArgs.Controller )
		.NumericTypeInterface(InArgs.NumericTypeInterface)
		.Visibility(InArgs.VisibilityDelegate)
		.ShowWorkingRange(!!(InArgs.ShowRanges & EShowRange::WorkingRange))
		.ShowViewRange(!!(InArgs.ShowRanges & EShowRange::ViewRange))
		.ShowPlaybackRange(!!(InArgs.ShowRanges & EShowRange::PlaybackRange))
		.EnableWorkingRange(!!(InArgs.EnableRanges & EShowRange::WorkingRange))
		.EnableViewRange(!!(InArgs.EnableRanges & EShowRange::ViewRange))
		.EnablePlaybackRange(!!(InArgs.EnableRanges & EShowRange::PlaybackRange))
		.PlaybackRangeStartColor(InArgs.PlaybackRangeStartColor)
		.PlaybackRangeEndColor(InArgs.PlaybackRangeEndColor)
		[
			Content
		];
	}

public:

	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }
};


IMPLEMENT_MODULE(FSequencerWidgetsModule, SequencerWidgets);
