// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetFocusUtils.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"

FPendingWidgetFocus::FPendingWidgetFocus(const TArray<FName>& InTypesKeepingFocus)
	: KeepingFocus(InTypesKeepingFocus)
{}

FPendingWidgetFocus FPendingWidgetFocus::MakeNoTextEdit()
{
	static TArray<FName> EditableTextTypes({"SEditableText"});
	// NOTE: "SMultiLineEditableText" might be added as well
	
	static FPendingWidgetFocus NewPendingFocus(EditableTextTypes);
	return NewPendingFocus;
}

FPendingWidgetFocus::~FPendingWidgetFocus()
{
	PendingFocusFunction.Reset();

	if (FSlateApplication::IsInitialized())
	{	
		FSlateApplication& SlateApplication = FSlateApplication::Get();
		if (PreInputKeyDownHandle.IsValid())
		{
			SlateApplication.OnApplicationPreInputKeyDownListener().Remove(PreInputKeyDownHandle);
		}
		if (PreInputButtonDownHandle.IsValid())
		{
			SlateApplication.OnApplicationMousePreInputButtonDownListener().Remove(PreInputButtonDownHandle);
		}
	}

	PreInputKeyDownHandle.Reset();
	PreInputButtonDownHandle.Reset();
}

void FPendingWidgetFocus::SetPendingFocusIfNeeded(const TWeakPtr<SWidget>& InWidget)
{
	if (!FSlateApplication::IsInitialized())
	{
		return;	
	}
	
	if (!IsEnabled())
	{
		return;
	}

	if (!CanFocusBeStolen())
	{
		PendingFocusFunction.Reset();
		return;
	}
	
	PendingFocusFunction = [WidgetFocus = InWidget]()
	{
		if (WidgetFocus.IsValid())
		{
			TSharedPtr<SWidget> Widget = WidgetFocus.Pin();
			FSlateApplication::Get().ForEachUser([&Widget](FSlateUser& User) 
			{
				User.SetFocus(Widget.ToSharedRef());
			});
		}
	};
}

void FPendingWidgetFocus::ResetPendingFocus()
{
	PendingFocusFunction.Reset();
}

void FPendingWidgetFocus::Enable(const bool InEnabled)
{
	if (!FSlateApplication::IsInitialized())
	{
		PendingFocusFunction.Reset();
		PreInputKeyDownHandle.Reset();
		PreInputButtonDownHandle.Reset();
		return;	
	}
	
	FSlateApplication& SlateApplication = FSlateApplication::Get();
	if (PreInputKeyDownHandle.IsValid())
	{
		SlateApplication.OnApplicationPreInputKeyDownListener().Remove(PreInputKeyDownHandle);
		PreInputKeyDownHandle.Reset();
	}
	if (PreInputButtonDownHandle.IsValid())
	{
		SlateApplication.OnApplicationMousePreInputButtonDownListener().Remove(PreInputButtonDownHandle);
		PreInputButtonDownHandle.Reset();
	}
	
	PendingFocusFunction.Reset();
	
	if (InEnabled)
	{
		PreInputKeyDownHandle = SlateApplication.OnApplicationPreInputKeyDownListener().AddRaw(this, &FPendingWidgetFocus::OnPreInputKeyDown);
		PreInputButtonDownHandle = SlateApplication.OnApplicationMousePreInputButtonDownListener().AddRaw(this, &FPendingWidgetFocus::OnPreInputButtonDown);
	}
}

bool FPendingWidgetFocus::IsEnabled() const
{
	return PreInputKeyDownHandle.IsValid() && PreInputButtonDownHandle.IsValid();
}
	
void FPendingWidgetFocus::OnPreInputKeyDown(const FKeyEvent&)
{
	if (PendingFocusFunction)
	{
		PendingFocusFunction();
		PendingFocusFunction.Reset();
	}
}

void FPendingWidgetFocus::OnPreInputButtonDown(const FPointerEvent&)
{
	// remove any pending focus as clicking a mouse button will set the focus
	// so this pending function should not interfere.
	PendingFocusFunction.Reset();
}

bool FPendingWidgetFocus::CanFocusBeStolen() const
{
	if (!KeepingFocus.IsEmpty())
	{
		bool bShouldCurrentFocusBeKept = false;

		if (FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().ForEachUser([this, &bShouldCurrentFocusBeKept](const FSlateUser& User)
		   {
			   if (TSharedPtr<SWidget> FocusedWidget = User.GetFocusedWidget())
			   {
				   bShouldCurrentFocusBeKept = KeepingFocus.Contains(FocusedWidget->GetType());
			   }
		   });
		}
		
		return !bShouldCurrentFocusBeKept;
	}
	return true;
}

