// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Range.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"

/** Structure used to wrap up a range, and an optional animation target */
struct FAnimatedRange : public TRange<double>
{
	/** Default Construction */
	FAnimatedRange() : TRange() {}
	/** Construction from a lower and upper bound */
	FAnimatedRange(double LowerBound, double UpperBound) : TRange(LowerBound, UpperBound) {}
	/** Copy-construction from simple range */
	FAnimatedRange(const TRange<double>& InRange) : TRange(InRange) {}

	/** Helper function to wrap an attribute to an animated range with a non-animated one */
	static TAttribute<TRange<double>> WrapAttribute(const TAttribute<FAnimatedRange>& InAttribute)
	{
		typedef TAttribute<TRange<double>> Attr;
		return Attr::Create(Attr::FGetter::CreateLambda([=]() { return InAttribute.Get(); }));
	}

	/** Helper function to wrap an attribute to a non-animated range with an animated one */
	static TAttribute<FAnimatedRange> WrapAttribute(const TAttribute<TRange<double>>& InAttribute)
	{
		typedef TAttribute<FAnimatedRange> Attr;
		return Attr::Create(Attr::FGetter::CreateLambda([=]() { return InAttribute.Get(); }));
	}

	/** Get the current animation target, or the whole view range when not animating */
	const TRange<double>& GetAnimationTarget() const
	{
		return AnimationTarget.IsSet() ? AnimationTarget.GetValue() : *this;
	}

	/** The animation target, if animating */
	TOptional<TRange<double>> AnimationTarget;
};
