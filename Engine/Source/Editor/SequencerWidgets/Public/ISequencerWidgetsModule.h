// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "ITimeSlider.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Modules/ModuleInterface.h"
#include "Misc/QualifiedFrameTime.h"

/** Enum denoting which time ranges to display on a time range */
enum class EShowRange
{
	None			= 0,
	WorkingRange	= 1 << 0,
	ViewRange		= 1 << 1,
	PlaybackRange	= 1 << 2,
};
ENUM_CLASS_FLAGS(EShowRange);

/** Time range construction arguments */
struct FTimeRangeArgs
{
	FTimeRangeArgs(
		EShowRange InShowRanges,
		TSharedRef<ITimeSliderController> InController,
		TAttribute<EVisibility> InVisibilityDelegate,
		TAttribute<TSharedPtr<INumericTypeInterface<double>>> InNumericTypeInterface,
		FLinearColor InPlaybackRangeStartColor = FLinearColor::Green,
		FLinearColor InPlaybackRangeEndColor = FLinearColor::Red
	)
		: ShowRanges(InShowRanges)
		, EnableRanges(InShowRanges)
		, Controller(InController)
		, VisibilityDelegate(InVisibilityDelegate)
		, NumericTypeInterface(InNumericTypeInterface)
		, PlaybackRangeStartColor(InPlaybackRangeStartColor)
		, PlaybackRangeEndColor(InPlaybackRangeEndColor)
	{
	}

	FTimeRangeArgs(
		EShowRange InShowRanges,
		EShowRange InEnableRanges,
		TSharedRef<ITimeSliderController> InController,
		TAttribute<EVisibility> InVisibilityDelegate,
		TAttribute<TSharedPtr<INumericTypeInterface<double>>> InNumericTypeInterface,
		FLinearColor InPlaybackRangeStartColor = FLinearColor::Green,
		FLinearColor InPlaybackRangeEndColor = FLinearColor::Red
		)
		: ShowRanges(InShowRanges)
		, EnableRanges(InEnableRanges)
		, Controller(InController)
		, VisibilityDelegate(InVisibilityDelegate)
		, NumericTypeInterface(InNumericTypeInterface)
		, PlaybackRangeStartColor(InPlaybackRangeStartColor)
		, PlaybackRangeEndColor(InPlaybackRangeEndColor)
	{
	}

	/** Which ranges to show */
	EShowRange ShowRanges;
	/** Which ranges to enable editing on */
	EShowRange EnableRanges;
	/** A time slider controller */
	TSharedRef<ITimeSliderController> Controller;
	/** Visibility delegate */
	TAttribute<EVisibility> VisibilityDelegate;
	/** Numeric type interface to use for frame<->time conversion */
	TAttribute<TSharedPtr<INumericTypeInterface<double>>> NumericTypeInterface;
	/** Playback range start color */
	FLinearColor PlaybackRangeStartColor = FLinearColor::Green;
	/** Playback range end color */
	FLinearColor PlaybackRangeEndColor = FLinearColor::Red;
};

/**
 * The public interface of SequencerModule
 */
class ISequencerWidgetsModule
	: public IModuleInterface
{
public:

	virtual TSharedRef<ITimeSlider> CreateTimeSlider( const TSharedRef<ITimeSliderController>& InController, bool bMirrorLabels  ) = 0;
	virtual TSharedRef<ITimeSlider> CreateTimeSlider( const TSharedRef<ITimeSliderController>& InController, const TAttribute<EVisibility>& VisibilityDelegate, bool bMirrorLabels  ) = 0;
	virtual TSharedRef<SWidget> CreateTimeRangeSlider( const TSharedRef<class ITimeSliderController>& InController ) = 0;
	virtual TSharedRef<ITimeSlider> CreateTimeRange( const FTimeRangeArgs& InArgs, const TSharedRef<SWidget>& Content ) = 0;
};
