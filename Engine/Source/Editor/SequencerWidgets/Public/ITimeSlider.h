// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerInputHandler.h"
#include "ViewRangeInterpolation.h"
#include "Widgets/SCompoundWidget.h"


class FSlateWindowElementList;
struct FSlateBrush;

struct FGeometry;
struct FFrameRate;
struct FAnimatedRange;
class FSlateRect;
class FWidgetStyle;

struct FPaintPlaybackRangeArgs
{
	FPaintPlaybackRangeArgs()
		: StartBrush(nullptr), EndBrush(nullptr), BrushWidth(0.f), SolidFillOpacity(0.f)
	{}

	FPaintPlaybackRangeArgs(const FSlateBrush* InStartBrush, const FSlateBrush* InEndBrush, float InBrushWidth)
		: StartBrush(InStartBrush), EndBrush(InEndBrush), BrushWidth(InBrushWidth)
	{}
	/** Brush to use for the start bound */
	const FSlateBrush* StartBrush;
	/** Brush to use for the end bound */
	const FSlateBrush* EndBrush;
	/** The width of the above brushes, in slate units */
	float BrushWidth;
	/** level of opacity for the fill color between the range markers */
	float SolidFillOpacity;
};

struct FPaintViewAreaArgs
{
	FPaintViewAreaArgs()
		: bDisplayTickLines(false), bDisplayScrubPosition(false), bDisplayMarkedFrames(false)
	{}

	/** Whether to display tick lines */
	bool bDisplayTickLines;
	/** Whether to display the scrub position */
	bool bDisplayScrubPosition;
	/** Whether to display the marked frames */
	bool bDisplayMarkedFrames;
	/** Optional Paint args for the playback range*/
	TOptional<FPaintPlaybackRangeArgs> PlaybackRangeArgs;
};

enum ETimeSliderPlaybackStatus 
{
	Stopped,
	Playing,
	Scrubbing,
	Jumping,
	Stepping,
	Paused,
	MAX
};

class ITimeSliderController : public ISequencerInputHandler
{
public:
	virtual ~ITimeSliderController() override = default;

	virtual int32 OnPaintTimeSlider( bool bMirrorLabels, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const = 0;

	virtual int32 OnPaintViewArea( const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, bool bEnabled, const FPaintViewAreaArgs& Args ) const = 0;

	virtual FReply OnTimeSliderMouseButtonDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		return OnMouseButtonDown(OwnerWidget, MyGeometry, MouseEvent);
	}

	virtual FReply OnTimeSliderMouseButtonUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		return OnMouseButtonUp(OwnerWidget, MyGeometry, MouseEvent);
	}

	virtual FReply OnTimeSliderMouseMove(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		return OnMouseMove(OwnerWidget, MyGeometry, MouseEvent);
	}

	virtual FReply OnTimeSliderMouseWheel(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		return OnMouseWheel(OwnerWidget, MyGeometry, MouseEvent);
	}
	
	virtual double ComputeHeight() const { return 22.0; };

	/** Get the current play rate for this controller */
	virtual FFrameRate GetDisplayRate() const = 0;

	/** Get the current tick resolution for this controller */
	virtual FFrameRate GetTickResolution() const = 0;

	/** Get the current view range for this controller */
	SEQUENCERWIDGETS_API virtual FAnimatedRange GetViewRange() const;

	/** Get the current clamp range for this controller */
	SEQUENCERWIDGETS_API virtual FAnimatedRange GetClampRange() const;

	/** Get the current play range for this controller */
	SEQUENCERWIDGETS_API virtual TRange<FFrameNumber> GetPlayRange() const;

	/** Get the current time bounds for this controller. The time bounds should be a subset of the playback range. */
	SEQUENCERWIDGETS_API virtual TRange<FFrameNumber> GetTimeBounds() const;

	/** Get the current selection range for this controller */
	SEQUENCERWIDGETS_API virtual TRange<FFrameNumber> GetSelectionRange() const;

	/** Get the current time for the Scrub handle which indicates what range is being evaluated. */
	SEQUENCERWIDGETS_API virtual FFrameTime GetScrubPosition() const;

	/** Set the current time for the Scrub handle which indicates what range is being evaluated. */
	SEQUENCERWIDGETS_API virtual void SetScrubPosition(FFrameTime InTime, bool bEvaluate);

	/** Set the current time for the handle when playback is stopped*/
	SEQUENCERWIDGETS_API virtual void SetStoppedPosition(FFrameTime InTime);
	
	/** Set the playback status for the controller*/
	SEQUENCERWIDGETS_API virtual void SetPlaybackStatus(ETimeSliderPlaybackStatus InStatus);

	/** Get the playback status for the controller, by default it is ETimeSliderPlaybackStatus::Stopped */
	SEQUENCERWIDGETS_API virtual ETimeSliderPlaybackStatus GetPlaybackStatus() const;


	/**
	 * Set a new range based on a min, max and an interpolation mode
	 * 
	 * @param NewRangeMin		The new lower bound of the range
	 * @param NewRangeMax		The new upper bound of the range
	 * @param Interpolation		How to set the new range (either immediately, or animated)
	 */
	SEQUENCERWIDGETS_API virtual void SetViewRange(double NewRangeMin, double NewRangeMax, EViewRangeInterpolation Interpolation);

	/**
	 * Set a new clamp range based on a min, max
	 * 
	 * @param NewRangeMin		The new lower bound of the clamp range
	 * @param NewRangeMax		The new upper bound of the clamp range
	 */
	SEQUENCERWIDGETS_API virtual void SetClampRange(double NewRangeMin, double NewRangeMax);

	/**
	 * Set a new playback range based on a min, max
	 * 
	 * @param RangeStart		The new lower bound of the playback range
	 * @param RangeDuration		The total number of frames that we play for
	 */
	SEQUENCERWIDGETS_API virtual void SetPlayRange(FFrameNumber RangeStart, int32 RangeDuration);

	/**
	 * Set a new selection range
	 * 
	 * @param NewRange		The new selection range
	 */
	SEQUENCERWIDGETS_API virtual void SetSelectionRange(const TRange<FFrameNumber>& NewRange);
};

/**
 * Base class for a widget that scrubs time or frames
 */
class ITimeSlider : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(ITimeSlider){}
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()
};
