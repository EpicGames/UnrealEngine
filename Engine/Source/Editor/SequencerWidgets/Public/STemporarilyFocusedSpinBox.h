// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWidget.h"
#include "Widgets/Input/SSpinBox.h"
#include "Framework/Application/SlateApplication.h"

/**
 * A widget that holds a widget that is to be refocused on completion
 */
template<typename T>
struct STemporarilyFocusedSpinBox : SSpinBox<T>
{
public:
	void Setup()
	{
		PreviousFocusedWidget = FSlateApplication::Get().GetKeyboardFocusedWidget();
	}

	void Refocus()
	{
		if (PreviousFocusedWidget.IsValid())
		{
			FSlateApplication::Get().SetKeyboardFocus(PreviousFocusedWidget.Pin());
		}
	}

private:
	TWeakPtr<SWidget> PreviousFocusedWidget;
};