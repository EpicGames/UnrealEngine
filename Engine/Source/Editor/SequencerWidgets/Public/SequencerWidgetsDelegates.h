// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Math/Range.h"
#include "ViewRangeInterpolation.h"

struct FFrameTime;
struct FFrameNumber;
struct FMovieSceneSequenceID;
enum class ENearestKeyOption : uint8;

DECLARE_DELEGATE_ThreeParams(FOnScrubPositionChanged, FFrameTime, bool, bool)
DECLARE_DELEGATE_TwoParams(FOnViewRangeChanged, TRange<double>, EViewRangeInterpolation)
DECLARE_DELEGATE_OneParam(FOnTimeRangeChanged, TRange<double>)
DECLARE_DELEGATE_OneParam(FOnFrameRangeChanged, TRange<FFrameNumber>)
DECLARE_DELEGATE_TwoParams(FOnSetMarkedFrame, int32, FFrameNumber)
DECLARE_DELEGATE_OneParam(FOnAddMarkedFrame, FFrameNumber)
DECLARE_DELEGATE_RetVal_TwoParams(FFrameNumber, FOnGetNearestKey, FFrameTime, ENearestKeyOption)
DECLARE_DELEGATE_OneParam(FOnScrubPositionParentChanged, FMovieSceneSequenceID)
