// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimatedRange.h"
#include "Delegates/Delegate.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "MovieSceneMarkedFrame.h"
#include "MovieSceneSequenceID.h"
#include "SequencerWidgetsDelegates.h"
#include "Widgets/Input/NumericTypeInterface.h"

namespace EMovieScenePlayerStatus
{
	enum Type : int;
}

/** Enum specifying how to find the nearest key */
enum class ENearestKeyOption : uint8
{
	NKO_None = 0x00,

	/* Search keys */
	NKO_SearchKeys = 0x01,

	/* Search sections */
	NKO_SearchSections = 0x02,

	/* Search markers */
	NKO_SearchMarkers = 0x04,

	/** Search all tracks */
	NKO_SearchAllTracks UE_DEPRECATED(5.4, "Search all tracks has been deprecated")  = 0x08
};
ENUM_CLASS_FLAGS(ENearestKeyOption);

struct FTimeSliderArgs
{
	FTimeSliderArgs()
		: ScrubPosition(0)
		, ViewRange(FAnimatedRange(0.0f, 5.0f))
		, ClampRange(FAnimatedRange(-FLT_MAX / 2.f, FLT_MAX / 2.f))
		, AllowZoom(true)
	{}

	/** The scrub position */
	TAttribute<FFrameTime> ScrubPosition;

	/** The scrub position text */
	TAttribute<FString> ScrubPositionText;

	/** The parent sequence that the scrub position display text is relative to */
	TAttribute<FMovieSceneSequenceID> ScrubPositionParent;

	/** Called when the scrub position parent sequence is changed */
	FOnScrubPositionParentChanged OnScrubPositionParentChanged;

	/** Attribute for the parent sequence chain of the current sequence */
	TAttribute<TArray<FMovieSceneSequenceID>> ScrubPositionParentChain;

	/** View time range */
	TAttribute< FAnimatedRange > ViewRange;

	/** Clamp time range */
	TAttribute< FAnimatedRange > ClampRange;

	/** Called when the scrub position changes */
	FOnScrubPositionChanged OnScrubPositionChanged;

	/** Called right before the scrubber begins to move */
	FSimpleDelegate OnBeginScrubberMovement;

	/** Called right after the scrubber handle is released by the user */
	FSimpleDelegate OnEndScrubberMovement;

	/** Called when the view range changes */
	FOnViewRangeChanged OnViewRangeChanged;

	/** Called when the clamp range changes */
	FOnTimeRangeChanged OnClampRangeChanged;

	/** Delegate that is called when getting the nearest key */
	FOnGetNearestKey OnGetNearestKey;

	/** Attribute defining the active sub-sequence range for this controller */
	TAttribute<TOptional<TRange<FFrameNumber>>> SubSequenceRange;

	/** Attribute defining the playback range for this controller */
	TAttribute<TRange<FFrameNumber>> PlaybackRange;

	/** Attribute defining the time bounds for this controller. The time bounds should be a subset of the playback range. */
	TAttribute<TRange<FFrameNumber>> TimeBounds;

	/** Attribute for the current sequence's display rate */
	TAttribute<FFrameRate> DisplayRate;

	/** Attribute for the current sequence's tick resolution */
	TAttribute<FFrameRate> TickResolution;

	/** Delegate that is called when the playback range wants to change */
	FOnFrameRangeChanged OnPlaybackRangeChanged;

	/** Called right before the playback range starts to be dragged */
	FSimpleDelegate OnPlaybackRangeBeginDrag;

	/** Called right after the playback range has finished being dragged */
	FSimpleDelegate OnPlaybackRangeEndDrag;

	/** Attribute defining the selection range for this controller */
	TAttribute<TRange<FFrameNumber>> SelectionRange;

	/** Delegate that is called when the selection range wants to change */
	FOnFrameRangeChanged OnSelectionRangeChanged;

	/** Called right before the selection range starts to be dragged */
	FSimpleDelegate OnSelectionRangeBeginDrag;

	/** Called right after the selection range has finished being dragged */
	FSimpleDelegate OnSelectionRangeEndDrag;

	/** Called right before a mark starts to be dragged */
	FSimpleDelegate OnMarkBeginDrag;

	/** Called right after a mark has finished being dragged */
	FSimpleDelegate OnMarkEndDrag;

	/** Attribute for the current sequence's vertical frames */
	TAttribute<TSet<FFrameNumber>> VerticalFrames;

	/** Attribute for the current sequence's marked frames */
	TAttribute<TArray<FMovieSceneMarkedFrame>> MarkedFrames;

	/** Attribute for the marked frames that might need to be shown, but do not belong to the current sequence*/
	TAttribute<TArray<FMovieSceneMarkedFrame>> GlobalMarkedFrames;

	/** Called when the marked frame needs to be set */
	FOnSetMarkedFrame OnSetMarkedFrame;

	/** Called when a marked frame is added */
	FOnAddMarkedFrame OnAddMarkedFrame;

	/** Called when all marked frames should be deleted */
	FSimpleDelegate OnDeleteAllMarkedFrames;

	/** Round the scrub position to an integer during playback */
	TAttribute<EMovieScenePlayerStatus::Type> PlaybackStatus;

	/** Attribute defining whether the playback range is locked */
	TAttribute<bool> IsPlaybackRangeLocked;

	/** Attribute defining whether the marked frames are locked */
	TAttribute<bool> AreMarkedFramesLocked;

	/** Attribute defining the time snap interval */
	TAttribute<float> TimeSnapInterval;

	/** Called when toggling the playback range lock */
	FSimpleDelegate OnTogglePlaybackRangeLocked;

	/** Called when toggling the marked frames lock */
	FSimpleDelegate OnToggleMarkedFramesLocked;

	/** If we are allowed to zoom */
	bool AllowZoom;

	/** Numeric Type interface for converting between frame numbers and display formats. */
	TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface;
};
