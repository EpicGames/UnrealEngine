// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** Enum specifying how to interpolate to a new view range */
enum class EViewRangeInterpolation
{
	/** Use an externally defined animated interpolation */
	Animated,
	/** Set the view range immediately */
	Immediate,
};
