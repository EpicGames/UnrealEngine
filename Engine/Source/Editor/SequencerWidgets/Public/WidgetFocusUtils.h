// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/Function.h"
#include "Delegates/IDelegateInstance.h"

#define UE_API SEQUENCERWIDGETS_API

class SWidget;
struct FKeyEvent;
struct FPointerEvent;

/**
 * FPendingWidgetFocus is a utility class that stores a pending focus function when a SWidget is hovered over.
 * It aims to provide a way to focus on a widget without having to actually click on it. 
 * The focus function is stored on the mouse enter event and will only be executed if a key down event is sent while the widget is hovered.
 * If the mouse leave event is called without any key down event having been called, the function is reset and the focus is not modified at all.
 */

class FPendingWidgetFocus : public TSharedFromThis<FPendingWidgetFocus>
{
public:

	FPendingWidgetFocus() = default;
	UE_API FPendingWidgetFocus(const TArray<FName>& InTypesKeepingFocus);

	static UE_API FPendingWidgetFocus MakeNoTextEdit();
	
	UE_API ~FPendingWidgetFocus();

	UE_API void Enable(const bool InEnabled);
	UE_API bool IsEnabled() const;

	UE_API void SetPendingFocusIfNeeded(const TWeakPtr<SWidget>& InWidget);
	UE_API void ResetPendingFocus();

private:
	
	UE_API void OnPreInputKeyDown(const FKeyEvent&);
	UE_API void OnPreInputButtonDown(const FPointerEvent&);
	UE_API bool CanFocusBeStolen() const;

	TFunction<void()> PendingFocusFunction;
	FDelegateHandle PreInputKeyDownHandle;
	FDelegateHandle PreInputButtonDownHandle;

	TArray<FName> KeepingFocus;
};

#undef UE_API
