// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class SequencerWidgets : ModuleRules
{
	public SequencerWidgets(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"SequencerCore",
			}
		   );

        PublicDependencyModuleNames.AddRange(
            new string[] {
                "MovieScene",
            }
        );
        
        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"InputCore",
				"Slate",
				"SlateCore",
			}
		);
	}
}
